// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class DeviceProfileServices : ModuleRules
{

	public DeviceProfileServices(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"Engine",
				"Json",
				"JsonUtilities",
			});

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"CoreUObject",
				"TargetPlatform",
				"DesktopPlatform",
				"EditorFramework",
				"UnrealEd",
				"PIEPreviewDeviceSpecification",
			});

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
			});
	}
}

