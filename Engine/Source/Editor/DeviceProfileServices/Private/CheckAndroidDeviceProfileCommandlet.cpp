// Copyright Epic Games, Inc. All Rights Reserved.

#include "CheckAndroidDeviceProfileCommandlet.h"

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "DeviceProfiles/DeviceProfile.h"
#include "DeviceProfiles/DeviceProfileManager.h"
#include "DeviceProfiles/DeviceProfileMatching.h"
#include "Dom/JsonObject.h"
#include "GenericPlatform/GenericPlatformMemory.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformCrt.h"
#include "IDeviceProfileSelectorModule.h"
#include "JsonObjectConverter.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CString.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/FileHelper.h"
#include "Modules/ModuleManager.h"
#include "PIEPreviewDeviceSpecification.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "Templates/SharedPointer.h"
#include "Templates/Tuple.h"
#include "Trace/Detail/Channel.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CheckAndroidDeviceProfileCommandlet)

DEFINE_LOG_CATEGORY_STATIC(LogCheckAndroidDeviceProfile, Log, All);

EPlatformMemorySizeBucket DetermineDeviceMemorySizeBucket(int32 TotalPhysicalMemoryGB)
{
	EPlatformMemorySizeBucket Bucket = EPlatformMemorySizeBucket::Default;
	static int32 LargestMemoryGB = 0, LargerMemoryGB = 0, DefaultMemoryGB = 0, SmallerMemoryGB = 0, SmallestMemoryGB = 0, TiniestMemoryGB = 0;
	static bool bTriedPlatformBuckets = false;
	if (!bTriedPlatformBuckets)
	{
		bTriedPlatformBuckets = true;
		FConfigFile EngineConfigFile;
		if (FConfigCacheIni::LoadLocalIniFile(EngineConfigFile, TEXT("Engine"), true, TEXT("Android")))
		{
			// get values for this platform from it's .ini
			EngineConfigFile.GetInt(TEXT("PlatformMemoryBuckets"), TEXT("LargestMemoryBucket_MinGB"), LargestMemoryGB);
			EngineConfigFile.GetInt(TEXT("PlatformMemoryBuckets"), TEXT("LargerMemoryBucket_MinGB"), LargerMemoryGB);
			EngineConfigFile.GetInt(TEXT("PlatformMemoryBuckets"), TEXT("DefaultMemoryBucket_MinGB"), DefaultMemoryGB);
			EngineConfigFile.GetInt(TEXT("PlatformMemoryBuckets"), TEXT("SmallerMemoryBucket_MinGB"), SmallerMemoryGB);
			EngineConfigFile.GetInt(TEXT("PlatformMemoryBuckets"), TEXT("SmallestMemoryBucket_MinGB"), SmallestMemoryGB);
			EngineConfigFile.GetInt(TEXT("PlatformMemoryBuckets"), TEXT("TiniestMemoryBucket_MinGB"), TiniestMemoryGB);
		}
	}

	// if at least Smaller is specified, we can set the Bucket
	if (SmallerMemoryGB > 0)
	{
		if (TotalPhysicalMemoryGB >= SmallerMemoryGB)
		{
			Bucket = EPlatformMemorySizeBucket::Smaller;
		}
		else if (TotalPhysicalMemoryGB >= SmallestMemoryGB)
		{
			Bucket = EPlatformMemorySizeBucket::Smallest;
		}
		else
		{
			Bucket = EPlatformMemorySizeBucket::Tiniest;
		}
	}
	if (DefaultMemoryGB > 0 && TotalPhysicalMemoryGB >= DefaultMemoryGB)
	{
		Bucket = EPlatformMemorySizeBucket::Default;
	}
	if (LargerMemoryGB > 0 && TotalPhysicalMemoryGB >= LargerMemoryGB)
	{
		Bucket = EPlatformMemorySizeBucket::Larger;
	}
	if (LargestMemoryGB > 0 && TotalPhysicalMemoryGB >= LargestMemoryGB)
	{
		Bucket = EPlatformMemorySizeBucket::Largest;
	}
	return Bucket;
}

int32 UCheckAndroidDeviceProfileCommandlet::Main(const FString& RawCommandLine)
{
	TArray<FString> Tokens;
	TArray<FString> Switches;
	TMap<FString, FString> Params;
	ParseCommandLine(*RawCommandLine, Tokens, Switches, Params);

	FString ParamDeviceSpecsFolder = Params.FindRef(TEXT("DeviceSpecsFolder"));
	FString ParamDeviceSpecsFile = Params.FindRef(TEXT("DeviceSpecsFile"));
	FString ForcedDeviceProfileName = Params.FindRef(TEXT("OverrideDP"));
	FString OutputFolder = Params.FindRef(TEXT("OutDir"));

	TArray<FString> DeviceSpecificationFileNames;
	if (!ParamDeviceSpecsFolder.IsEmpty())
	{
		IFileManager::Get().FindFiles(DeviceSpecificationFileNames, *(ParamDeviceSpecsFolder / TEXT("*.json")), true, false);
	}
	else if (!ParamDeviceSpecsFile.IsEmpty())
	{
		DeviceSpecificationFileNames.Add(ParamDeviceSpecsFile);
	}

	FName DeviceProfileSelectorModule("AndroidDeviceProfileSelector");
	IDeviceProfileSelectorModule* AndroidDeviceProfileSelector = FModuleManager::LoadModulePtr<IDeviceProfileSelectorModule>(DeviceProfileSelectorModule);
	if (ensure(AndroidDeviceProfileSelector != nullptr))
	{
		do
		{
			TMap<FString, FString> DeviceParameters;
			FString DeviceName;
			FString OutputFilename;
			if (DeviceSpecificationFileNames.Num())
			{
				FString DeviceSpecsFile = DeviceSpecificationFileNames.Pop();
				int32 ExtensionPos;
				if (DeviceSpecsFile.FindLastChar(TEXT('.'), ExtensionPos))
				{
					DeviceName = DeviceSpecsFile.Left(ExtensionPos);
					OutputFilename = DeviceName;
				}

				FString JsonLocation = ParamDeviceSpecsFolder / DeviceSpecsFile;
				AndroidDeviceProfileSelector->GetDeviceParametersFromJson(JsonLocation, DeviceParameters);
			}
			else
			{
				DeviceName = TEXT("[no name]");
				DeviceParameters.Add(TEXT("SRC_GPUFamily"), Params.FindRef(TEXT("GPUFamily")));
				DeviceParameters.Add(TEXT("SRC_GLVersion"), Params.FindRef(TEXT("GLVersion")));
				DeviceParameters.Add(TEXT("SRC_VulkanAvailable"), Params.FindRef(TEXT("VulkanAvailable")));
				DeviceParameters.Add(TEXT("SRC_VulkanVersion"), Params.FindRef(TEXT("VulkanVersion")));
				DeviceParameters.Add(TEXT("SRC_AndroidVersion"), Params.FindRef(TEXT("AndroidVersion")));
				DeviceParameters.Add(TEXT("SRC_DeviceMake"),
					Tokens.Num() == 2 ? Tokens[0] :
					Params.FindRef(TEXT("DeviceMake")));
				DeviceParameters.Add(TEXT("SRC_DeviceModel"),
					Tokens.Num() == 1 ? Tokens[0] :
					Tokens.Num() == 2 ? Tokens[1] :
					Params.FindRef(TEXT("DeviceModel")));
				DeviceParameters.Add(TEXT("SRC_DeviceBuildNumber"), Params.FindRef(TEXT("DeviceBuildNumber")));
				DeviceParameters.Add(TEXT("SRC_UsingHoudini"), Params.FindRef(TEXT("UsingHoudini")));
				DeviceParameters.Add(TEXT("SRC_Hardware"), Params.FindRef(TEXT("Hardware")));
				DeviceParameters.Add(TEXT("SRC_Chipset"), Params.FindRef(TEXT("Chipset")));
				DeviceParameters.Add(TEXT("SRC_TotalPhysicalGB"), Params.FindRef(TEXT("TotalPhysicalGB")));
				DeviceParameters.Add(TEXT("SRC_HMDSystemName"), TEXT(""));
				DeviceParameters.Add(TEXT("SRC_SM5Available"), Params.FindRef(TEXT("SM5Available")));
				DeviceParameters.Add(TEXT("SRC_ResolutionX"), Params.FindRef(TEXT("ResolutionX")));
				DeviceParameters.Add(TEXT("SRC_ResolutionY"), Params.FindRef(TEXT("ResolutionY")));
				DeviceParameters.Add(TEXT("SRC_InsetsLeft"), Params.FindRef(TEXT("InsetsLeft")));
				DeviceParameters.Add(TEXT("SRC_InsetsTop"), Params.FindRef(TEXT("InsetsTop")));
				DeviceParameters.Add(TEXT("SRC_InsetsRight"), Params.FindRef(TEXT("InsetsRight")));
				DeviceParameters.Add(TEXT("SRC_InsetsBottom"), Params.FindRef(TEXT("InsetsBottom")));
				DeviceParameters.Add(TEXT("SRC_VKQuality"), Params.FindRef(TEXT("VKQuality")));
			}
			AndroidDeviceProfileSelector->SetSelectorProperties(DeviceParameters);
			FString ProfileName = ForcedDeviceProfileName.IsEmpty() ? AndroidDeviceProfileSelector->GetDeviceProfileName() : ForcedDeviceProfileName;
			TArray<FSelectedFragmentProperties> SelectedFragments;

			UE_LOGF(LogCheckAndroidDeviceProfile, Display, "%ls Selected Device Profile: %ls", *DeviceName, *ProfileName);
			int32 TotalPhysGB = FCString::Atoi(*DeviceParameters.FindChecked(TEXT("SRC_TotalPhysicalGB")));
			EPlatformMemorySizeBucket MemorySizeBucket = DetermineDeviceMemorySizeBucket(TotalPhysGB);
			FString PlatformOverride(TEXT("Android"));
			FString ProfileDescription;
			UDeviceProfile* Profile = UDeviceProfileManager::Get().FindProfile(ProfileName, false);
			TArray<FString> CVars;
			if (!Profile)
			{
				UE_LOGF(LogCheckAndroidDeviceProfile, Error, "Failed to find requested device profile. %ls requested Device Profile: %ls But could not be found!", *DeviceName, *ProfileName);
			}
			else
			{
				// Set the memory size bucket for this device.
				Profile->SetPreviewMemorySizeBucket(MemorySizeBucket);
				// ensure any previous expanded cvars are cleared out.
				Profile->ClearAllExpandedCVars();
				// get all GetAllExpandedCVars re-runs the DP selection.
				TMap<FString, FString> CVarMap = Profile->GetAllExpandedCVars();

				for (auto CVarIt : CVarMap)
				{
					CVars.Add(FString::Printf(TEXT("%s=%s"), *CVarIt.Key, *CVarIt.Value));
				}
				SelectedFragments = Profile->SelectedFragments;
			}

			if (!OutputFilename.IsEmpty())
			{
				TStringBuilder<0> Output;

				if (!DeviceName.IsEmpty())
				{
					Output.Appendf(TEXT("Device name : %s\n\n"), *DeviceName);
				}
				if (!Profile)
				{
					Output.Appendf(TEXT("Failed to find device profile : %s\n\n"), *ProfileName);
				}

				OutputFilename = DeviceName + TEXT("_") + DeviceParameters.FindChecked(TEXT("SRC_GPUFamily")) + TEXT("_") + DeviceParameters.FindChecked(TEXT("SRC_TotalPhysicalGB")) + TEXT("GB.txt");
				Output.Appendf(TEXT("AndroidDeviceProfileSelector Params:\n"));
				TArray<FString> SortedKeys;
				DeviceParameters.GenerateKeyArray(SortedKeys);
				//SortedKeys.Sort();
				for (const FString& ParamKey : SortedKeys)
				{
					Output.Appendf(TEXT("[%s]=%s\n"), *ParamKey, *DeviceParameters.FindChecked(ParamKey));
				}
				Output.Appendf(TEXT("\nDevice Mem Bucket: %s\n"), LexToString(MemorySizeBucket));

				Output.Appendf(TEXT("\nSelected fragments :\n%s"), SelectedFragments.Num() == 0 ? TEXT("-none-\n") : TEXT(""));
				for (FSelectedFragmentProperties& Fragment : SelectedFragments)
				{
					FString FragTag;
					if (Fragment.Tag != NAME_None)
					{
						FragTag = TEXT("[") + Fragment.Tag.ToString() + TEXT("]");
					}
					Output.Appendf(TEXT("%s%s : Enabled=%s\n"), *FragTag, *Fragment.Fragment, Fragment.bEnabled ? TEXT("true") : TEXT("false"));
				}

				Output.Appendf(TEXT("\n\nSelected device Profile: %s\nSelected device profile description: %s\n"), *ProfileName, *ProfileDescription);

				// may not want this:
				const bool bSortCVars = true;
				if (bSortCVars)
				{
					Output.Appendf(TEXT("Sorted "));
					CVars.Sort();
				}
				FString CVarsOnly = FString::Printf(TEXT("CVars:\n"));

				for (FString& CVar : CVars)
				{
					CVarsOnly += FString::Printf(TEXT("%s\n"), *CVar);
				}
				Output.Append(CVarsOnly);
				FFileHelper::SaveStringToFile(Output, *(OutputFolder / OutputFilename));

				FFileHelper::SaveStringToFile(CVarsOnly, *(OutputFolder / TEXT("CVarsOnly") / OutputFilename));
			}

		} while (DeviceSpecificationFileNames.Num());
	}

	return 0;
}
