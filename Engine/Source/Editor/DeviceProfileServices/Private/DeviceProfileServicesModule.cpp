// Copyright Epic Games, Inc. All Rights Reserved.

#include "DeviceProfileServicesUIManager.h"
#include "HAL/PlatformCrt.h"
#include "IDeviceProfileServicesModule.h"
#include "IDeviceProfileServicesUIManager.h"
#include "Modules/ModuleManager.h"
#include "Templates/SharedPointer.h"


/**
 * Implements the DeviceProfileServices module.
 */
class FDeviceProfileServicesModule
	: public IDeviceProfileServicesModule
{
public:

	// IDeviceProfileServicesModule interface

	virtual IDeviceProfileServicesUIManagerRef GetProfileServicesManager( ) override
	{
		if (!DeviceProfileServicesUIManagerSingleton.IsValid())
		{
			DeviceProfileServicesUIManagerSingleton = MakeShareable(new FDeviceProfileServicesUIManager());
		}

		return DeviceProfileServicesUIManagerSingleton.ToSharedRef();
	}

protected:

	// Holds the session manager singleton.
	static IDeviceProfileServicesUIManagerPtr DeviceProfileServicesUIManagerSingleton;
};


/* Static initialization
 *****************************************************************************/

IDeviceProfileServicesUIManagerPtr FDeviceProfileServicesModule::DeviceProfileServicesUIManagerSingleton = NULL;

IMPLEMENT_MODULE(FDeviceProfileServicesModule, DeviceProfileServices);
