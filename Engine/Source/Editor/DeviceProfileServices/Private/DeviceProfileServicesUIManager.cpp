// Copyright Epic Games, Inc. All Rights Reserved.

#include "DeviceProfileServicesUIManager.h"

#include "CoreGlobals.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "DeviceProfiles/DeviceProfile.h"
#include "DeviceProfiles/DeviceProfileManager.h"
#include "HAL/PlatformCrt.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "Logging/LogMacros.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreMisc.h"
#include "Misc/DataDrivenPlatformInfoRegistry.h"
#include "Templates/Casts.h"
#include "UObject/UnrealNames.h"


DEFINE_LOG_CATEGORY_STATIC(LogDeviceProfileServices, Log, All);


FDeviceProfileServicesUIManager::FDeviceProfileServicesUIManager()
{
	UDeviceProfileManager::Get().OnManagerUpdated().AddRaw(this, &FDeviceProfileServicesUIManager::HandleRefreshUIData);
	HandleRefreshUIData();
	CreatePlatformMap();
}


const FName FDeviceProfileServicesUIManager::GetDeviceIconName( const FString& DeviceName ) const
{
	FName IconName = NAME_None;

	const FString* PlatformName = DeviceToPlatformMap.Find( DeviceName );
	if ( PlatformName )
	{
		IconName = GetPlatformIconName( *PlatformName );
	}

	return IconName;
}


const TArray<TSharedPtr<FString> > FDeviceProfileServicesUIManager::GetPlatformList()
{
	return PlatformList;
}


void FDeviceProfileServicesUIManager::GetProfilesByType( TArray<UDeviceProfile*>& OutDeviceProfiles, const FString& InType )
{
	for( int32 Idx = 0; Idx < UDeviceProfileManager::Get().Profiles.Num(); Idx++ )
	{
		UDeviceProfile* CurrentDevice = CastChecked<UDeviceProfile>( UDeviceProfileManager::Get().Profiles[Idx] );
		if ( CurrentDevice->DeviceType == InType )
		{
			OutDeviceProfiles.Add( CurrentDevice );
		}
	}
}


const FName FDeviceProfileServicesUIManager::GetPlatformIconName( const FString& PlatformName ) const
{
	const FName* PlatformNameTemp = DeviceTypeToIconMap.Find( PlatformName );
	if ( PlatformNameTemp )
	{
		return *PlatformNameTemp;
	}
	return NAME_None;
}


void FDeviceProfileServicesUIManager::HandleRefreshUIData()
{
	// Rebuild profile to platform map
	DeviceToPlatformMap.Empty();
	for( int32 Idx = 0; Idx < UDeviceProfileManager::Get().Profiles.Num(); Idx++ )
	{
		UDeviceProfile* CurrentDevice = CastChecked<UDeviceProfile>( UDeviceProfileManager::Get().Profiles[Idx] );
		DeviceToPlatformMap.Add( CurrentDevice->GetName(), CurrentDevice->DeviceType );
	}
}


void FDeviceProfileServicesUIManager::CreatePlatformMap()
{
	PlatformList.Reset();
	DeviceTypeToIconMap.Empty();

	TArray<ITargetPlatform*> Platforms = GetTargetPlatformManager()->GetTargetPlatforms();
	for (int32 Index = 0; Index < Platforms.Num(); ++Index)
	{
		PlatformList.Add(MakeShareable(new FString(Platforms[Index]->IniPlatformName())));
		DeviceTypeToIconMap.Add( Platforms[Index]->PlatformName(), Platforms[Index]->GetPlatformInfo().GetIconStyleName(EPlatformIconSize::Normal) );
	}
}


void FDeviceProfileServicesUIManager::SetProfile( const FString& DeviceProfileName )
{
	// Save the profile name to an ini file
	if ( DeviceProfileName != TEXT( "Default" ) )
	{
		const FString INISection = "SelectedProfile"; // Section in the game ini to store our selections
		const FString INIKeyBase = "ProfileItem"; // Key to the stored device profiles
		const int32 MaxItems = 4; // Max history

		// Array to store the existing items
		TArray< FString > CurItems;
		FString CurItem;

		// Get the existing items
		for( int32 ItemIdx = 0 ; ItemIdx < MaxItems; ++ItemIdx )
		{
			if ( GConfig->GetString( *INISection, *FString::Printf( TEXT("%s%d"), *INIKeyBase, ItemIdx ), CurItem, GEditorPerProjectIni ) )
			{
				CurItems.Add( CurItem );
			}
		}

		// Remove the current item if it exists - we will re-add it at the top of the array later
		const int32 ItemIndex = CurItems.Find( DeviceProfileName );
		if ( ItemIndex != INDEX_NONE )
		{
			CurItems.RemoveAt( ItemIndex );
		}
		else if ( CurItems.Num() == MaxItems )
		{
			// else remove the last item
			CurItems.RemoveAt( MaxItems -1 );
		}

		// Add the new profile to the top of the array
		CurItems.Insert( DeviceProfileName, 0 );

		// Clear the ini section
		GConfig->EmptySection( *INISection, GEditorPerProjectIni );

		// Re-write the .ini file
		for ( int32 ItemIdx = 0; ItemIdx < CurItems.Num(); ++ItemIdx )
		{
			GConfig->SetString( *INISection, *FString::Printf( TEXT("%s%d"), *INIKeyBase, ItemIdx ), *CurItems[ItemIdx], GEditorPerProjectIni );
		}

		GConfig->Flush( false, GEditorPerProjectIni );
	}
}
