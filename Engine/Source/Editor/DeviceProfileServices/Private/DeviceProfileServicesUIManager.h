// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "IDeviceProfileServicesUIManager.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

class UClass;
class UDeviceProfile;

/**
 * Implements the device profile services manager for UI
 */
class FDeviceProfileServicesUIManager
	: public IDeviceProfileServicesUIManager
{
public:

	/** Default constructor. */
	FDeviceProfileServicesUIManager( );

	/** Destructor. */
	virtual ~FDeviceProfileServicesUIManager() = default;

public:

	// IDeviceProfileServicesUIManager Interface

	virtual const FName GetDeviceIconName( const FString& DeviceName ) const override;
	virtual const TArray<TSharedPtr<FString> > GetPlatformList( ) override;
	virtual void GetProfilesByType( TArray<UDeviceProfile*>& OutDeviceProfiles, const FString& InType ) override;
	virtual const FName GetPlatformIconName( const FString& DeviceName ) const override;
	virtual void SetProfile( const FString& DeviceProfileName ) override;

protected:

	/** Generates the UI icon list. */
	void CreatePlatformMap();

	/** Refresh the UI list - rebuild lists. */
	void HandleRefreshUIData();

private:

	/** Map of profiles to platform types. */
	TMap<const UClass*, FString> PickerTypeMap;

	/** Map of profiles to platform types. */
	TMap<const FString, FString> DeviceToPlatformMap;

	/** Map of profiles to platform types. */
	TMap<const FString, FName> DeviceTypeToIconMap;

	/** Holds the list of known platforms. */
	TArray<TSharedPtr<FString> > PlatformList;
};
