// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Commandlets/Commandlet.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "CheckAndroidDeviceProfileCommandlet.generated.h"

class UObject;

/*
 * CheckAndroidDeviceProfile
 *   Commandlet that runs the rules in [/Script/AndroidDeviceProfileSelector.AndroidDeviceProfileMatchingRules]
 *      against the supplied parameters and outputs the device profile that will be matched.
 * Usage:
 * ProjectName -run=DeviceProfileServices.CheckAndroidDeviceProfile DeviceModel
 * ProjectName -run=DeviceProfileServices.CheckAndroidDeviceProfile DeviceMake DeviceModel
 * ProjectName -run=DeviceProfileServices.CheckAndroidDeviceProfile [-DeviceMake=...] [-DeviceModel=...] [-GPUFamily=...] [-GLVersion=...]
	  [-VulkanAvailable=True|False] [-VulkanVersion=...] [-AndroidVersion=...] [-DeviceBuildNumber=...] [-UsingHoudini=True|False] [-Hardware=...] [-Chipset=...]
 * Running against 
 * ProjectName -run=DeviceProfileServices.CheckAndroidDeviceProfile
		-DeviceSpecsFolder=<directory containing device.json files> 
		-DeviceSpecsFile=<path to a single device.json file>
		-OutDir=<output directory>
		[-OverrideDP=<optional DP name to override device profile selection>]
*/
UCLASS()
class UCheckAndroidDeviceProfileCommandlet
	: public UCommandlet
{
	GENERATED_BODY()
public:

	//~ Begin UCommandlet Interface

	virtual int32 Main(const FString& Params) override;

	//~ End UCommandlet Interface
private:

};
