// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


/* Boilerplate
 *****************************************************************************/

#include "Misc/MonolithicHeaderBoilerplate.h"
MONOLITHIC_HEADER_BOILERPLATE()


/* Dependencies
 *****************************************************************************/

#include "Core.h"
#include "Engine.h"
#include "Modules/ModuleInterface.h"


/* Interfaces
 *****************************************************************************/

#include "IDeviceProfileServicesUIManager.h"
#include "IDeviceProfileServicesModule.h"
