// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class IDeviceProfileServicesUIManager;
class UDeviceProfile;

/** Type definition for shared pointers to instances of IDeviceProfileServicesUIManager. */
typedef TSharedPtr<class IDeviceProfileServicesUIManager> IDeviceProfileServicesUIManagerPtr;

/** Type definition for shared references to instances of IDeviceProfileServicesUIManager. */
typedef TSharedRef<class IDeviceProfileServicesUIManager> IDeviceProfileServicesUIManagerRef;


/**
 * Device Profile Services Manager for UI.
 */
class IDeviceProfileServicesUIManager
{
public:

	/**
	 * Gets an icon name for a given device.
	 *
	 * @param DeviceName The device name.
	 * @return The icon name.
	 */
	virtual const FName GetDeviceIconName( const FString& DeviceName ) const = 0;

	/**
	 * Gets Platform List.
	 *
	 * @return the platform list.
	 */
	virtual const TArray<TSharedPtr<FString> > GetPlatformList( ) = 0;

	/**
	 * Gets an array of profile that match a type.
	 *
	 * @param OutDeviceProfiles The populated profile list.
	 * @param InType The type to match.
	 */
	virtual void GetProfilesByType( TArray<UDeviceProfile*>& OutDeviceProfiles, const FString& InType ) = 0;

	/**
	 * Gets an icon name for a given platform.
	 *
	 * @param DeviceName -The platform name.
	 * @return The icon name.
	 */
	virtual const FName GetPlatformIconName( const FString& DeviceName ) const = 0;

	/**
	 * Set a profile - add the type to an INI file so it can be used in a history view.
	 *
	 * @param DeviceProfileName The platform type.
	 */
	virtual void SetProfile( const FString& DeviceProfileName ) = 0;
};
